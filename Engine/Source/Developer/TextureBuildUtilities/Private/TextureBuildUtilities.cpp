// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureBuildUtilities.h"

#include "Async/TaskGraphInterfaces.h"
#include "ImageCoreUtils.h"
#include "TextureCompressorModule.h" // for FTextureBuildSettings
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
//#include "EngineLogs.h" // can't use from SCW

DEFINE_LOG_CATEGORY_STATIC(LogTextureBuildUtilities, Log, All);

namespace UE
{
namespace TextureBuildUtilities
{

// Return true if texture format name is HDR
TEXTUREBUILDUTILITIES_API bool TextureFormatIsHdr(FName const& InName)
{
	// TextureFormatRemovePrefixFromName first !
	
	static FName NameRGBA16F(TEXT("RGBA16F"));
	static FName NameRGBA32F(TEXT("RGBA32F"));
	static FName NameR16F(TEXT("R16F"));
	static FName NameR32F(TEXT("R32F"));
	static FName NameBC6H(TEXT("BC6H"));

	if ( InName == NameRGBA16F ) return true;
	if ( InName == NameRGBA32F ) return true;
	if ( InName == NameR16F ) return true;
	if ( InName == NameR32F ) return true;
	if ( InName == NameBC6H ) return true;

	return false;
}

TEXTUREBUILDUTILITIES_API const FName TextureFormatRemovePlatformPrefixFromName(FName const& InName)
{
	FString NameString = InName.ToString();

	// Format names may have one of the following forms:
	// - PLATFORM_PREFIX_FORMAT
	// - PLATFORM_FORMAT
	// - PREFIX_FORMAT
	// - FORMAT
	// We have to remove the platform prefix first, if it exists.
	// Then we detect a non-platform prefix (such as codec name)
	// and split the result into  explicit FORMAT and PREFIX parts.

	// fast(ish) early out if there are no underscores in InName :
	int32 UnderscoreIndexIgnored = INDEX_NONE;
	if ( ! NameString.FindChar(TCHAR('_'), UnderscoreIndexIgnored))
	{
		return InName;
	}

	for (FName PlatformName : FDataDrivenPlatformInfoRegistry::GetSortedPlatformNames(EPlatformInfoType::AllPlatformInfos))
	{
		FString PlatformTextureFormatPrefix = PlatformName.ToString();
		PlatformTextureFormatPrefix += TEXT('_');
		if (NameString.StartsWith(PlatformTextureFormatPrefix, ESearchCase::IgnoreCase))
		{
			// Remove platform prefix and proceed with non-platform prefix detection.
			FString PlatformRemoved = NameString.RightChop(PlatformTextureFormatPrefix.Len());
			return FName( PlatformRemoved );
		}
	}
	
	return InName;
}
	
TEXTUREBUILDUTILITIES_API const FName TextureFormatRemovePrefixFromName(FName const& InNameWithPlatform, FName& OutPrefix)
{
	// first remove platform prefix :
	FName NameWithoutPlatform = TextureFormatRemovePlatformPrefixFromName( InNameWithPlatform );
	FString NameString = NameWithoutPlatform.ToString();

	// then see if there's another underscore separated prefix :
	int32 UnderscoreIndex = INDEX_NONE;
	if ( ! NameString.FindChar(TCHAR('_'), UnderscoreIndex))
	{
		return NameWithoutPlatform;
	}

	// texture format names can have underscores in them (eg. ETC2_RG11)
	//	so need to differentiate between that and a conditional prefix :

	// found an underscore; is it a composite texture name, or an "Alternate" prefix?
	FString Prefix = NameString.Left(UnderscoreIndex + 1);
	if ( Prefix == "OODLE_" || Prefix == "TFO_" )
	{
		// Alternate prefix
		OutPrefix = FName( Prefix );
		return FName( NameString.RightChop(UnderscoreIndex + 1) );
	}
	else if ( Prefix == "ASTC_" || Prefix == "ETC2_" )
	{
		// composite format, don't split
		return NameWithoutPlatform;
	}
	else
	{
		// prefix not recognized
		// LogTexture doesn't exist in SCW
		UE_LOGF(LogCore,Warning,"Texture Format Prefix not recognized: %ls [%ls]",*Prefix,*InNameWithPlatform.ToString());
		
		return NameWithoutPlatform;
	}
}


TEXTUREBUILDUTILITIES_API ERawImageFormat::Type GetVirtualTextureBuildIntermediateFormat(const FTextureBuildSettings& BuildSettings)
{
	// Platform prefix should have already been removed, also remove any Oodle prefix:
	const FName TextureFormatName = TextureFormatRemovePrefixFromName(BuildSettings.TextureFormatName);

	// note: using RGBA16F when the Source is HDR but the output is not HDR is not needed
	//	you could use BGRA8 intermediate in that case
	//	but it's rare and not a big problem, so leave it alone for now

	const bool bIsHdr = BuildSettings.bHDRSource || TextureFormatIsHdr(TextureFormatName);

	if (bIsHdr)
	{
		return ERawImageFormat::RGBA16F;
	}
	else if ( TextureFormatName == "G16" )
	{
		return ERawImageFormat::G16;
	}
	else
	{
		return ERawImageFormat::BGRA8;
	}
}

#ifndef TEXT_TO_ENUM
#define TEXT_TO_ENUM(eVal, txt)		if (txt.Compare(#eVal) == 0)	return eVal;
#endif

static EPixelFormat GetPixelFormatFromUtf8(const FUtf8StringView& InPixelFormatStr)
{
#define TEXT_TO_PIXELFORMAT(f) TEXT_TO_ENUM(f, InPixelFormatStr);
	FOREACH_ENUM_EPIXELFORMAT_CORE(TEXT_TO_PIXELFORMAT)
#undef TEXT_TO_PIXELFORMAT
		return PF_Unknown;
}


namespace EncodedTextureExtendedData
{
	FCbObject ToCompactBinary(const FEncodedTextureExtendedData& InExtendedData)
	{
		FCbWriter Writer;
		Writer.BeginObject();
		Writer.AddInteger("NumMipsInTail", InExtendedData.NumMipsInTail);
		Writer.AddInteger("ExtData", InExtendedData.ExtData);
		Writer.BeginArray("MipSizes");
		for (uint64 MipSize : InExtendedData.MipSizesInBytes)
		{
			Writer.AddInteger(MipSize);
		}
		Writer.EndArray();
		Writer.EndObject();
		return Writer.Save().AsObject();
	}

	bool FromCompactBinary(FEncodedTextureExtendedData& OutExtendedData, FCbObject InCbObject)
	{
		OutExtendedData.ExtData = InCbObject["ExtData"].AsUInt32();
		OutExtendedData.NumMipsInTail = InCbObject["NumMipsInTail"].AsInt32();

		FCbArrayView MipArrayView = InCbObject["MipSizes"].AsArrayView();
		for (FCbFieldView MipFieldView : MipArrayView)
		{
			OutExtendedData.MipSizesInBytes.Add(MipFieldView.AsUInt64());
		}
		return true;
	}
} // namespace EncodedTextureExtendedData

namespace EncodedTextureDescription
{
	FCbObject ToCompactBinary(const FEncodedTextureDescription& InDescription)
	{
		FCbWriter Writer;
		Writer.BeginObject();
		Writer.AddInteger("TopMipSizeX", InDescription.TopMipSizeX);
		Writer.AddInteger("TopMipSizeY", InDescription.TopMipSizeY);
		Writer.AddInteger("TopMipVolumeSizeZ", InDescription.TopMipVolumeSizeZ);
		Writer.AddInteger("ArraySlices", InDescription.ArraySlices);
		Writer.AddString("PixelFormat", GetPixelFormatString(InDescription.PixelFormat));
		Writer.AddInteger("NumMips", InDescription.NumMips);
		Writer.AddBool("bCubeMap", InDescription.bCubeMap);
		Writer.AddBool("bTextureArray", InDescription.bTextureArray);
		Writer.AddBool("bVolumeTexture", InDescription.bVolumeTexture);
		Writer.EndObject();
		return Writer.Save().AsObject();
	}

	bool FromCompactBinary(FEncodedTextureDescription& OutDescription, FCbObject InCbObject)
	{
		OutDescription.TopMipSizeX = InCbObject["TopMipSizeX"].AsInt32();
		OutDescription.TopMipSizeY = InCbObject["TopMipSizeY"].AsInt32();
		OutDescription.TopMipVolumeSizeZ = InCbObject["TopMipVolumeSizeZ"].AsInt32();
		OutDescription.ArraySlices = InCbObject["ArraySlices"].AsInt32();
		OutDescription.PixelFormat = GetPixelFormatFromUtf8(InCbObject["PixelFormat"].AsString());
		OutDescription.NumMips = (uint8)InCbObject["NumMips"].AsInt32();
		OutDescription.bCubeMap = InCbObject["bCubeMap"].AsBool();
		OutDescription.bTextureArray = InCbObject["bTextureArray"].AsBool();
		OutDescription.bVolumeTexture = InCbObject["bVolumeTexture"].AsBool();
		return true;
	}
} // namespace EncodedTextureDescription



namespace TextureEngineParameters
{
	FCbObject ToCompactBinaryWithDefaults(const FTextureEngineParameters& InEngineParameters)
	{
		FTextureEngineParameters Defaults;

		FCbWriter Writer;
		Writer.BeginObject();
		if (InEngineParameters.bEngineSupportsTexture2DArrayStreaming != Defaults.bEngineSupportsTexture2DArrayStreaming)
		{
			Writer.AddBool("bEngineSupportsTexture2DArrayStreaming", InEngineParameters.bEngineSupportsTexture2DArrayStreaming);
		}
		if (InEngineParameters.bEngineSupportsVolumeTextureStreaming != Defaults.bEngineSupportsVolumeTextureStreaming)
		{
			Writer.AddBool("bEngineSupportsVolumeTextureStreaming", InEngineParameters.bEngineSupportsVolumeTextureStreaming);
		}
		if (InEngineParameters.NumInlineDerivedMips != Defaults.NumInlineDerivedMips)
		{
			Writer.AddInteger("NumInlineDerivedMips", InEngineParameters.NumInlineDerivedMips);
		}
		Writer.EndObject();
		return Writer.Save().AsObject();
	}

	bool FromCompactBinary(FTextureEngineParameters& OutEngineParameters, FCbObject InCbObject)
	{
		OutEngineParameters = FTextureEngineParameters(); // init to defaults

		OutEngineParameters.NumInlineDerivedMips = InCbObject["NumInlineDerivedMips"].AsInt32(OutEngineParameters.NumInlineDerivedMips);
		OutEngineParameters.bEngineSupportsTexture2DArrayStreaming = InCbObject["bEngineSupportsTexture2DArrayStreaming"].AsBool(OutEngineParameters.bEngineSupportsTexture2DArrayStreaming);
		OutEngineParameters.bEngineSupportsVolumeTextureStreaming = InCbObject["bEngineSupportsVolumeTextureStreaming"].AsBool(OutEngineParameters.bEngineSupportsVolumeTextureStreaming);
		return true;
	}
} // namespace EncodedTextureDescription

FCbObject FTextureBuildMetadata::ToCompactBinaryWithDefaults() const
{
	FTextureBuildMetadata Defaults;

	FCbWriter Writer;
	Writer.BeginObject();
	if (PreEncodeMipsHash != Defaults.PreEncodeMipsHash)
	{
		Writer << UTF8TEXTVIEW("PreEncodeMipsHash") << PreEncodeMipsHash;
	}
	Writer.EndObject();
	return Writer.Save().AsObject();
}

FTextureBuildMetadata::FTextureBuildMetadata(FCbObject InCbObject)
{
	PreEncodeMipsHash = InCbObject["PreEncodeMipsHash"].AsUInt64(PreEncodeMipsHash);
}

void GetPlaceholderTextureImageInfo(FImageInfo* OutImageInfo)
{
	OutImageInfo->SizeX = 4;
	OutImageInfo->SizeY = 4;
	OutImageInfo->GammaSpace = EGammaSpace::sRGB;
	OutImageInfo->Format = ERawImageFormat::BGRA8;
	OutImageInfo->NumSlices = 1;
}
void GetPlaceholderTextureImage(FImage* OutImage)
{
	*OutImage = FImage();

	GetPlaceholderTextureImageInfo(OutImage);
	OutImage->RawData.AddUninitialized(sizeof(FColor) * OutImage->SizeX * OutImage->SizeY);
	for (FColor& Color : OutImage->AsBGRA8())
	{
		Color = FColor::Black;
	}

}


// Returns true if the target texture size is different and padding/stretching is required.
TEXTUREBUILDUTILITIES_API bool GetPowerOfTwoTargetTextureSize(int32 InMip0SizeX, int32 InMip0SizeY, int32 InMip0NumSlices, bool bInIsVolume, ETexturePowerOfTwoSetting::Type InPow2Setting, int32 InResizeDuringBuildX, int32 InResizeDuringBuildY, int32& OutTargetSizeX, int32& OutTargetSizeY, int32& OutTargetSizeZ)
{
	int32 TargetTextureSizeX = InMip0SizeX;
	int32 TargetTextureSizeY = InMip0SizeY;
	int32 TargetTextureSizeZ = bInIsVolume ? InMip0NumSlices : 1; // Only used for volume texture.

	const int32 PowerOfTwoTextureSizeX = FMath::RoundUpToPowerOfTwo(TargetTextureSizeX);
	const int32 PowerOfTwoTextureSizeY = FMath::RoundUpToPowerOfTwo(TargetTextureSizeY);
	const int32 PowerOfTwoTextureSizeZ = FMath::RoundUpToPowerOfTwo(TargetTextureSizeZ);

	switch (InPow2Setting)
	{
	case ETexturePowerOfTwoSetting::None:
		break;

	case ETexturePowerOfTwoSetting::PadToPowerOfTwo:
	case ETexturePowerOfTwoSetting::StretchToPowerOfTwo:
		TargetTextureSizeX = PowerOfTwoTextureSizeX;
		TargetTextureSizeY = PowerOfTwoTextureSizeY;
		TargetTextureSizeZ = PowerOfTwoTextureSizeZ;
		break;

	case ETexturePowerOfTwoSetting::PadToSquarePowerOfTwo:
	case ETexturePowerOfTwoSetting::StretchToSquarePowerOfTwo:
		TargetTextureSizeX = TargetTextureSizeY = TargetTextureSizeZ =
			FMath::Max3<int32>(PowerOfTwoTextureSizeX, PowerOfTwoTextureSizeY, PowerOfTwoTextureSizeZ);
		break;

	case ETexturePowerOfTwoSetting::ResizeToSpecificResolution:
		if (InResizeDuringBuildX)
		{
			TargetTextureSizeX = InResizeDuringBuildX;
		}
		if (InResizeDuringBuildY)
		{
			TargetTextureSizeY = InResizeDuringBuildY;
		}
		break;

	default:
		checkf(false, TEXT("Unknown entry in ETexturePowerOfTwoSetting::Type"));
		break;
	}

	// Z only matters as a sampling dimension if we are a volume texture.
	if (bInIsVolume == false)
	{
		TargetTextureSizeZ = InMip0NumSlices;
	}

	OutTargetSizeX = TargetTextureSizeX;
	OutTargetSizeY = TargetTextureSizeY;
	OutTargetSizeZ = TargetTextureSizeZ;

	return (TargetTextureSizeX != InMip0SizeX) ||
		(TargetTextureSizeY != InMip0SizeY) ||
		(bInIsVolume && TargetTextureSizeZ != InMip0NumSlices);
}

TEXTUREBUILDUTILITIES_API bool TextureNeedsDecodeForPC(EPixelFormat InPixelFormat, int32 InCreateMip0SizeX, int32 InCreateMip0SizeY)
{
	if (RequiresBlock4Alignment(InPixelFormat))
	{
		// DX requires this on the top mip that we create, not that we build necessarily
		if (InCreateMip0SizeX % 4 ||
			InCreateMip0SizeY % 4)
		{
			return true;
		}
	}

	// Check if we can render the pixel format on a texture.
	// We assume if we have texture2d we have all we need.
	return !EnumHasAnyFlags(GPixelFormats[InPixelFormat].Capabilities, EPixelFormatCapabilities::Texture2D);
}


static int GetWithinSliceRDOMemoryUsePerPixel(EPixelFormat PixelFormat)
{
	// Memory use of RDO data structures, per pixel, within each slice
	// not counting per-image memory use
	const int MemUse_BC1 = 57;
	const int MemUse_BC4 = 90;
	const int MemUse_BC5 = 2 * MemUse_BC4;
	const int MemUse_BC6 = 8;
	const int MemUse_BC7 = 30;
	const int MemUse_BC3 = MemUse_BC4; // max of BC1,BC4

	switch (PixelFormat)
	{
	case PF_DXT1:
		return MemUse_BC1;
	case PF_DXT3:
	case PF_DXT5:
		return MemUse_BC3;
	case PF_BC4:
		return MemUse_BC4;
	case PF_BC5:
		return MemUse_BC5;
	case PF_BC6H:
		return MemUse_BC6;
	case PF_BC7:
		return MemUse_BC7;
	default:
		// is this possible?
		UE_CALL_ONCE([&]() {
			UE_LOGF(LogTextureBuildUtilities, Display, "Unexpected non-BC PixelFormat: %d.", (int)PixelFormat);
		});

		return 100;
	}
}

static int64 CalculateTextureSourceBytesFromImageInfo(const FImageInfo & InImageInfoConst, const int32 InMipCountConst, const bool bInVolume)
{
	FImageInfo ImageInfo = InImageInfoConst;
	int32 MipCount = InMipCountConst;

	int64 SourceBytes = 0;
	while (MipCount)
	{
		SourceBytes += ImageInfo.GetImageSizeBytes();

		ImageInfo.SizeX = FEncodedTextureDescription::GetMipWidth(ImageInfo.SizeX, 1);
		ImageInfo.SizeY = FEncodedTextureDescription::GetMipHeight(ImageInfo.SizeY, 1);
		if (bInVolume)
		{
			ImageInfo.NumSlices = FEncodedTextureDescription::GetMipDepth(ImageInfo.NumSlices, 1, true);
		}
		MipCount--;
	}

	return SourceBytes;
}


TEXTUREBUILDUTILITIES_API EPixelFormat GetOutputPixelFormatWithFallback(const FTextureBuildSettings& InBuildSettings, bool bInKnownAlphaFallback)
{
	if (InBuildSettings.BaseTextureFormat == nullptr)
	{
		return PF_Unknown;
	}

	bool bHasAlpha = false;
	InBuildSettings.GetOutputAlphaFromKnownAlphaOrFallback(&bHasAlpha, bInKnownAlphaFallback);

	// If we get called and we have not stripped any platform prefix, this will crash if it's not TextureFormatOodle because
	// the others still look at TextureFormatName instead of BaseTextureFormatName.
	bool bNeedsBaseCopy = false;
	if (InBuildSettings.TextureFormatName != InBuildSettings.BaseTextureFormatName)
	{
		// Welp, we are different. If it's TextureFormatOodle then we are actually OK.
		// TextureFormatNames are pretty short so we can just copy out:
		TStringBuilder<32> FormatName;
		FormatName << InBuildSettings.BaseTextureFormatName;
		if (FormatName.Len() < 4 ||
			FCString::Strncmp(FormatName.GetData(), TEXT("TFO_"), 4))
		{
			// We aren't TFO and we are different so make a copy of us.
			bNeedsBaseCopy = true;
		}
	}

	if (InBuildSettings.BaseTextureFormat == nullptr)
	{
		return PF_Unknown;
	}

	EPixelFormat PixelFormat = PF_Unknown;
	if (bNeedsBaseCopy)
	{
		// Base texture formats expect to get TextureFormatName without the platform prefix.
		// We could call through the non-base TextureFormat but all it's doing is this:
		// eventually we'll migrate all texture formats to reference BaseTextureFormatName.
		FTextureBuildSettings BaseTextureBuildSettings = InBuildSettings;
		BaseTextureBuildSettings.TextureFormatName = BaseTextureBuildSettings.BaseTextureFormatName;
		PixelFormat = InBuildSettings.BaseTextureFormat->GetEncodedPixelFormat(BaseTextureBuildSettings, bHasAlpha);
	}
	else
	{
		PixelFormat = InBuildSettings.BaseTextureFormat->GetEncodedPixelFormat(InBuildSettings, bHasAlpha);
	}
	check(PixelFormat != PF_Unknown);

	return PixelFormat;
}


TEXTUREBUILDUTILITIES_API int64 GetVirtualTextureRequiredMemoryEstimate(const FTextureBuildSettings* InBuildSettingsPerLayer,
	TConstArrayView<ERawImageFormat::Type> InLayerFormats,
	TConstArrayView<UE::TextureBuildUtilities::FVirtualTextureSourceBlockInfo> InSourceBlocks)
{
	const bool bRDO = true;
	// @todo Oodle : be careful about using BuildSettings for bRDO as there are two buildsettingses, just assume its on for now
	//   <- FIX ME, allow lower mem estimates for non-RDO

	// over-estimate is okay
	// try not to over-estimate by too much (reduces parallelism of cook)

	int64 MaxNumberOfWorkers = FMath::Max(1, FTaskGraphInterface::Get().GetNumWorkerThreads());

	// VT build does :
	// load all source images
	// for each layer/block :
	//    generate mips (requires F32 copy)
	//    output to intermediate format
	//    intermediate format copy is then used to make tiles
	//    for each tile :
	//       make padded tile in intermediate format
	//       encode to output format
	//       discard padded tile in intermediate format
	// all output tiles are then aggregated

	// Compute the memory it should take to uncompress the bulkdata in memory
	int64 TotalSourceBytes = 0;
	int64 TotalTopMipNumPixelsPerLayer = 0;
	int64 LargestBlockTopMipNumPixels = 0;

	int64 ResizingPhaseMemUsePerLayer = 0;

	// All layers in a VT must have the same layout for each block. Layers only can change source pixels + format, not dims.
	for (int32 BlockIndex = 0; BlockIndex < InSourceBlocks.Num(); ++BlockIndex)
	{
		const UE::TextureBuildUtilities::FVirtualTextureSourceBlockInfo& SourceBlock = InSourceBlocks[BlockIndex];

		for (int32 LayerIndex = 0; LayerIndex < InLayerFormats.Num(); ++LayerIndex)
		{
			// Create an FImageInfo so we can calcualte size off of it.
			FImageInfo LayerBlock;
			LayerBlock.GammaSpace = EGammaSpace::Linear; // doesn't matter for size
			LayerBlock.Format = InLayerFormats[LayerIndex];
			LayerBlock.SizeX = SourceBlock.SizeX;
			LayerBlock.SizeY = SourceBlock.SizeY;
			LayerBlock.NumSlices = SourceBlock.NumSlices;

			TotalSourceBytes += CalculateTextureSourceBytesFromImageInfo(LayerBlock, SourceBlock.NumMips, false);
		}

		// assume pow2 options are the same for all layers, just use layer 0 here :
		const FTextureBuildSettings& LayerBuildSettings = InBuildSettingsPerLayer[0];

		check( ! LayerBuildSettings.bVolume );
		check( ! LayerBuildSettings.bCubemap );
		check( ! LayerBuildSettings.bLongLatSource );

		int32 TargetSizeX, TargetSizeY, TargetSizeZ;
		bool bDidPow2 = UE::TextureBuildUtilities::GetPowerOfTwoTargetTextureSize(SourceBlock.SizeX, SourceBlock.SizeY, SourceBlock.NumSlices,
			LayerBuildSettings.bVolume, (ETexturePowerOfTwoSetting::Type)LayerBuildSettings.PowerOfTwoMode,
			LayerBuildSettings.ResizeDuringBuildX, LayerBuildSettings.ResizeDuringBuildY,
			TargetSizeX, TargetSizeY, TargetSizeZ);
		
		int64 AfterPow2TopMipNumPixels = (int64) TargetSizeX * TargetSizeY * TargetSizeZ;
			
		// MaxTextureSize on UDIM applies to each block on its own
		if ( LayerBuildSettings.MaxTextureResolution != TNumericLimits<uint32>::Max() )
		{
			// max memory use of the MaxTextureResolution op is the source in RGBA32F + a mip of size /2 in RGBA32F
			ResizingPhaseMemUsePerLayer += AfterPow2TopMipNumPixels * 20;  // (1 + 1/4) * 16;
		
			// ResizingPhaseMemUse is per Layer at this point

			while( (uint32)TargetSizeX > LayerBuildSettings.MaxTextureResolution || (uint32)TargetSizeY > LayerBuildSettings.MaxTextureResolution )
			{
				TargetSizeX = FMath::Max(TargetSizeX>>1,1);
				TargetSizeY = FMath::Max(TargetSizeY>>1,1);

				// TargetSizeZ not changed
				check( ! LayerBuildSettings.bVolume );
			}
		}
		else if ( bDidPow2 )
		{
			// We create a copy of the source in RGBA32F as part of resizing
			// for some formats (FirstSourceMipImage "convert to RGBA32F")
			int64 SourceBlockNumPixels = (int64)SourceBlock.SizeX * SourceBlock.SizeY * SourceBlock.NumSlices;
			ResizingPhaseMemUsePerLayer += SourceBlockNumPixels * 16;
			// surface input to remaining processing in RGBA32F
			ResizingPhaseMemUsePerLayer += AfterPow2TopMipNumPixels * 16;
		}

		// TargetSize is now the size after Pow2 and MaxTextureSize resizes :

		int64 CurrentBlockTopMipNumPixels = (int64)TargetSizeX * TargetSizeY * TargetSizeZ;

		TotalTopMipNumPixelsPerLayer += CurrentBlockTopMipNumPixels;

		LargestBlockTopMipNumPixels = FMath::Max(CurrentBlockTopMipNumPixels, LargestBlockTopMipNumPixels);
	}

	if (TotalSourceBytes <= 0)
	{
		return -1; /* Unknown */
	}

	int64 ResizingPhaseMemUse = TotalSourceBytes + ResizingPhaseMemUsePerLayer * InLayerFormats.Num();

	// after this point, "numpixels" is the number encode to VT and output pixel format

	// assume full mip chain :
	int64 TotalPixelsPerLayer = (TotalTopMipNumPixelsPerLayer * 4) / 3;

	int64 TotalNumPixels = TotalPixelsPerLayer * InLayerFormats.Num();

	// only one block of one layer does the float image mip build at a time :
	int64 IntermediateFloatColorBytes = (LargestBlockTopMipNumPixels * sizeof(FLinearColor) * 4) / 3;

	int64 TileSize = InBuildSettingsPerLayer[0].VirtualTextureTileSize;
	int64 BorderSize = InBuildSettingsPerLayer[0].VirtualTextureBorderSize;

	int64 NumTilesPerLayer = FMath::DivideAndRoundUp<int64>(TotalPixelsPerLayer, TileSize * TileSize);
	int64 NumTiles = NumTilesPerLayer * InLayerFormats.Num();
	int64 TilePixels = (TileSize + 2 * BorderSize) * (TileSize + 2 * BorderSize);

	int64 NumOutputPixelsPerLayer = NumTilesPerLayer * TilePixels;

	// intermediate is created just once per block, use max size estimate
	int64 VTIntermediateSizeBytes = IntermediateFloatColorBytes;
	int64 OutputSizeBytes = 0;

	int64 MaxPerPixelEncoderMemUse = 0;

	for (int32 LayerIndex = 0; LayerIndex < InLayerFormats.Num(); ++LayerIndex)
	{
		const FTextureBuildSettings& LayerBuildSettings = InBuildSettingsPerLayer[LayerIndex];

		// VT builds to an intermediate format.

		ERawImageFormat::Type IntermediateImageFormat = UE::TextureBuildUtilities::GetVirtualTextureBuildIntermediateFormat(LayerBuildSettings);

		int64 IntermediateBytesPerPixel = ERawImageFormat::GetBytesPerPixel(IntermediateImageFormat);

		// + output bytes? (but can overlap with IntermediateFloatColorBytes)
		//	almost always less than IntermediateFloatColorBytes
		//  exception would be lots of udim blocks + lots of layers
		//  because IntermediateFloatColorBytes is per block/layer but output is held for all

		EPixelFormat PixelFormat = UE::TextureBuildUtilities::GetOutputPixelFormatWithFallback(LayerBuildSettings, true);

		if (PixelFormat == PF_Unknown)
		{
			return -1; /* Unknown */
		}

		const FPixelFormatInfo& PFI = GPixelFormats[PixelFormat];

		OutputSizeBytes += (NumOutputPixelsPerLayer * PFI.BlockBytes) / (PFI.BlockSizeX * PFI.BlockSizeY);

		// is it a blocked format :
		if (PFI.BlockSizeX > 1)
		{
			// another copy of Intermediate in BlockSurf swizzle :
			int CurPerPixelEncoderMemUse = IntermediateBytesPerPixel;

			if (bRDO)
			{
				int RDOMemUse = GetWithinSliceRDOMemoryUsePerPixel(PixelFormat);
				CurPerPixelEncoderMemUse += 4; // activity
				CurPerPixelEncoderMemUse += RDOMemUse;
				CurPerPixelEncoderMemUse += 1; // output again
			}

			// max over any layer :
			MaxPerPixelEncoderMemUse = FMath::Max(MaxPerPixelEncoderMemUse, CurPerPixelEncoderMemUse);
		}
	}

	// after we make the Intermediate layer, it is cut into tiles
	// we then need mem for the intermediate format padded up to tiles
	// and then working encoder mem & compressed output space for each tile
	//	(tiles are made one by one in the ParallelFor to make the compressed output)
	// but at that point the FloatColorBytes is freed

	int64 NumberOfWorkingTiles = FMath::Min(NumTiles, MaxNumberOfWorkers);

	// VT tile encode mem :  
	int64 MemoryUsePerTile = MaxPerPixelEncoderMemUse * TilePixels; // around 1.8 MB
	{
		 // MemoryUsePerTile
		 // makes tile in IntermediateBytesPerPixel
		 // encodes out to OutputSizeBytes
		 // encoder (Oodle) temp mem
		 // TilePixels * IntermediateBytesPerPixel (twice: surf+blocksurf)
		 // TilePixels * Output bytes (twice: baseline+rdo output) (output already counted)
		 // TilePixels * activity mask
		 // MaxPerPixelEncoderMemUse is around 100
	}

	int64 TileCompressionBytes = NumberOfWorkingTiles * MemoryUsePerTile;

	int64 MemoryEstimate = TotalSourceBytes + VTIntermediateSizeBytes;
	// @todo Oodle : After we make the VT Intermediate, is the source BulkData freed?
	//   -> it seems no at the moment, but it could be

	// take larger of mem use during float image filter phase or tile compression phase
	MemoryEstimate += FMath::Max(IntermediateFloatColorBytes, TileCompressionBytes + OutputSizeBytes);

	// larger of early resize phase and VT build phase :
	MemoryEstimate = FMath::Max(ResizingPhaseMemUse,MemoryEstimate);

	MemoryEstimate += 1024 * 1024; // overhead room

	//UE_LOGF(LogTextureBuildUtilities,Display,"GetBuildRequiredMemoryEstimate VT : %.3f MB",MemoryEstimate/(1024*1024.f));

	return MemoryEstimate;
}


// Returns the estimated memory cost of building the given texture. Only valid for physical (i.e. non-virtual) textures.
TEXTUREBUILDUTILITIES_API int64 GetPhysicalTextureBuildMemoryEstimate(const FTextureBuildSettings* InSettingsPerLayerFetchFirst, const FImageInfo& InSourceImageInfo, int32 InMipCount)
{
	if (InSettingsPerLayerFetchFirst->BaseTextureFormat == nullptr)
	{
		// Will fail build later, return no memory estimate
		return -1;
	}

	const bool bRDO = true;
	// @todo Oodle : be careful about using BuildSettings for bRDO as there are two buildsettingses, just assume its on for now
	//   <- FIX ME, allow lower mem estimates for non-RDO

	// over-estimate is okay
	// try not to over-estimate by too much (reduces parallelism of cook)

	int64 MaxNumberOfWorkers = FMath::Max(1, FTaskGraphInterface::Get().GetNumWorkerThreads());

	const FTextureBuildSettings& BuildSettings = InSettingsPerLayerFetchFirst[0];
	// non VT

	// Compute the memory it should take to uncompress the bulkdata in memory
	int64 TotalSourceBytes = CalculateTextureSourceBytesFromImageInfo(InSourceImageInfo, InMipCount, BuildSettings.bVolume);
	if (TotalSourceBytes <= 0)
	{
		return -1; /* Unknown */
	}

	// NOTE: it would be ideal to call Texture::GetBuiltTextureSize here, but we don't have a Texture pointer, sigh.

	int32 TargetSizeX, TargetSizeY, TargetSizeZ;
	bool bDidPow2 = UE::TextureBuildUtilities::GetPowerOfTwoTargetTextureSize(InSourceImageInfo.SizeX, InSourceImageInfo.SizeY, InSourceImageInfo.NumSlices,
		BuildSettings.bVolume, (ETexturePowerOfTwoSetting::Type)BuildSettings.PowerOfTwoMode,
		BuildSettings.ResizeDuringBuildX, BuildSettings.ResizeDuringBuildY,
		TargetSizeX, TargetSizeY, TargetSizeZ);
	
	int64 ResizingPhaseMemUse = TotalSourceBytes;

	// Pow2 resize can end up converting the *source* data to RGBA32F, so we need to account for it
	if (bDidPow2)
	{
		//  FirstSourceMipImage "convert to RGBA32F" to FImage Temp in TextureCompressorModule
		int64 SourceDataMipNumPixels = (int64)InSourceImageInfo.SizeX * InSourceImageInfo.SizeY * InSourceImageInfo.NumSlices;
		ResizingPhaseMemUse += SourceDataMipNumPixels * 16; // original source data in RGBA32F
		// This is live concurrently with the top mip in RGBA32F computed next.
		// Therefore we need the sum of both, not the max.
	}

	int64 InitialTopMipNumPixels = (int64) TargetSizeX * TargetSizeY * TargetSizeZ;
	ResizingPhaseMemUse += InitialTopMipNumPixels * 16; // top mip in RGBA32F may be needed

	if ( BuildSettings.bLongLatSource )
	{
		// longlat to cube is after pow2 pad
		TargetSizeX = TargetSizeY = ComputeLongLatCubemapExtents(TargetSizeX,BuildSettings.MaxTextureResolution);

		// could be a cube array :
		TargetSizeZ = InSourceImageInfo.NumSlices * 6;

		// mem use of the longlat->cube operation; requires source longlat in RGBA32F and the cube output :
		ResizingPhaseMemUse += (int64) TargetSizeX * TargetSizeY * TargetSizeZ * 16;
	}
	else if ( BuildSettings.MaxTextureResolution != TNumericLimits<uint32>::Max() )
	{
		// apply MaxTextureResolution
		// NOTE: it would be ideal to call Texture::GetBuiltTextureSize here, but we don't have a Texture pointer, sigh.
		//		(or some kind of shared function rather than duplicating all this logic)

		// max memory use of the MaxTextureResolution op is the source in RGBA32F + a mip of size /2 in RGBA32F
		ResizingPhaseMemUse += (InitialTopMipNumPixels/4) * 16;
		
		while( (uint32)TargetSizeX > BuildSettings.MaxTextureResolution || (uint32)TargetSizeY > BuildSettings.MaxTextureResolution )
		{
			TargetSizeX = FMath::Max(TargetSizeX>>1,1);
			TargetSizeY = FMath::Max(TargetSizeY>>1,1);

			if ( BuildSettings.bVolume )
			{
				TargetSizeZ = FMath::Max(TargetSizeZ>>1,1);
			}
		}
	}

	// from here on, NumPixels is the number that will be encoded to the output pixel format

	int64 TotalTopMipNumPixels = (int64)TargetSizeX * TargetSizeY * TargetSizeZ;

	// assume full mip chain :
	//	(volume mips are smaller than this, but over-estimating is okay)
	int64 TotalNumPixels = (TotalTopMipNumPixels * 4) / 3;

	// actually we have each mip twice for the float image filter phase so this is under-counting
	//	but that isn't held allocated while the output is made, so it can overlap with that mem
	int64 IntermediateFloatColorBytes = TotalNumPixels * sizeof(FLinearColor);

	// if we knew the source BulkData was always freed during encoding, TotalSourceBytes could be dropped
	int64 MemoryEstimate = TotalSourceBytes + IntermediateFloatColorBytes;

	// Assume alpha exists if we don't know for worst-case handling.
	const bool bHasAlphaFallback = true;
	EPixelFormat PixelFormat = GetOutputPixelFormatWithFallback(BuildSettings, bHasAlphaFallback);

	if (PixelFormat == PF_Unknown)
	{
		return -1; /* Unknown */
	}

	const FPixelFormatInfo& PFI = GPixelFormats[PixelFormat];

	const int64 OutputSizeBytes = (TotalNumPixels * PFI.BlockBytes) / (PFI.BlockSizeX * PFI.BlockSizeY);

	MemoryEstimate += OutputSizeBytes;

	// check to see if it's uncompressed or a BCN format :
	if (IsDXTCBlockCompressedTextureFormat(PixelFormat))
	{
		// block-compressed format ; assume it's using Oodle Texture

		if (bRDO)
		{
			// two more copies in outputsize
			// baseline encode + UT or Layout
			MemoryEstimate += OutputSizeBytes * 2;
		}

		// you also have to convert the float surface to an input format for Oodle
		//	this copy is done in TFO
		//  Oodle then allocs another copy to swizzle into blocks before encoding

		int IntermediateBytesPerPixel;
		bool bNeedsIntermediateCopy = true;

		// this matches the logic in TextureFormatOodle :
		if (PixelFormat == PF_BC6H)
		{
			IntermediateBytesPerPixel = 16; //RGBAF32
			bNeedsIntermediateCopy = false; // no intermediate used in TFO (float source kept), 1 blocksurf
		}
		else if (PixelFormat == PF_BC4 || PixelFormat == PF_BC5)
		{
			// changed: TFO uses 2_U16 now (4 byte intermediate)
			IntermediateBytesPerPixel = 8; // RGBA16
		}
		else
		{
			IntermediateBytesPerPixel = 4; // RGBA8
		}

		int NumIntermediateCopies = 1; // BlockSurf
		if (bNeedsIntermediateCopy) NumIntermediateCopies++;

		MemoryEstimate += NumIntermediateCopies * IntermediateBytesPerPixel * TotalNumPixels;

		if (bRDO)
		{
			// activity map for whole image :
			// (this has changed in newer versions of Oodle Texture)

			// Phase1 = computing activity map
			int ActivityBytesPerPixel;

			if (PixelFormat == PF_BC4) ActivityBytesPerPixel = 12;
			else if (PixelFormat == PF_BC5) ActivityBytesPerPixel = 16;
			else ActivityBytesPerPixel = 24;

			int64 RDOPhase1MemUse = ActivityBytesPerPixel * TotalNumPixels;

			// Phase2 = cut into slices, encode each slice
			// per-slice data structure memory use
			// non-RDO is all on stack so zero

			// fewer workers for small images ; roughly one slice per 64 KB of output
			//int64 NumberofSlices = FMath::DivideAndRoundUp<int64>(OutputSizeBytes,64*1024);
			int64 PixelsPerSlice = (64 * 1024 * TotalNumPixels) / OutputSizeBytes;
			int64 NumberofSlices = FMath::DivideAndRoundUp<int64>(TotalNumPixels, PixelsPerSlice);
			if (NumberofSlices <= 4)
			{
				PixelsPerSlice = TotalNumPixels / NumberofSlices;
			}

			int64 MemoryUsePerWorker = PixelsPerSlice * GetWithinSliceRDOMemoryUsePerPixel(PixelFormat);
				// MemoryUsePerWorker is around 10 MB
			int64 NumberOfWorkers = FMath::Min(NumberofSlices, MaxNumberOfWorkers);

			int64 RDOPhase2MemUse = 4 * TotalNumPixels; // activity map held on whole image
			RDOPhase2MemUse += NumberOfWorkers * MemoryUsePerWorker;

			// usually phase2 is higher
			// but on large BC6 images on machines with low core counts, phase1 can be higher

			MemoryEstimate += FMath::Max(RDOPhase1MemUse, RDOPhase2MemUse);
		}
	}
	else if (IsASTCBlockCompressedTextureFormat(PixelFormat))
	{
		// ASTCenc does an entermediate copy to RGBA16F for HDR formats and RGBA8 for LDR
		MemoryEstimate += (IsHDR(PixelFormat) ? 8 : 4) * TotalNumPixels;
			
		// internal memory use of ASTCenc :
		//	measured from command line astcenc.exe
		MemoryEstimate += 10 * TotalNumPixels;
	}
	else if ( PFI.BlockSizeX > 1 )
	{
		// block compressed but not Oodle or ASTC (eg. ETC)
		// note: memory ues of non-Oodle encoders is not estimated
		// @todo : fix me
			
		// prefer over-estimate to under-estimate :
		MemoryEstimate += 16 * TotalNumPixels;
	}
	else
	{
		// non-blocked encoder (uncompressed)
			
		// some of the TextureFormatUncompressed encoders use a scratch image
		//	must over-estimate to be safe :
		MemoryEstimate += 4 * TotalNumPixels;
	}

	// mem use is the max of the phases :
	MemoryEstimate = FMath::Max(MemoryEstimate,ResizingPhaseMemUse);

	MemoryEstimate += 1024 * 1024; // overhead room

	//UE_LOGF(LogTextureBuildUtilities,Display,"GetBuildRequiredMemoryEstimate non-VT : %.3f MB",MemoryEstimate/(1024*1024.f));

	return MemoryEstimate;

	// @todo Oodle : not right with Composite 
	
	// this is not right for CPU textures, but it is an over-estimate, so that's okay

	// note: this is intended to be right for TFO, not OTF
	//	 the cloud TBW and ContentWorker runs that really care about mem use limitations are TFO only
}

uint32 ComputeLongLatCubemapExtents(int32 SrcImageSizeX, uint32 MaxCubemapTextureResolution)
{
	// MaxTextureSize of 0 is changed to max when filling BuildSettings
	if ( MaxCubemapTextureResolution == 0 )
	{
		MaxCubemapTextureResolution = TNumericLimits<uint32>::Max();
	}

	uint32 Out = 1U << FMath::FloorLog2(SrcImageSizeX / 2);

	if ( Out <= 32 || MaxCubemapTextureResolution <= 32 )
	{
		return 32;
	}
	else if ( Out > MaxCubemapTextureResolution )
	{
		// RoundDownToPowerOfTwo
		return 1U << FMath::FloorLog2(MaxCubemapTextureResolution);
	}
	else
	{
		return Out;
	}
}

} // namespace
}