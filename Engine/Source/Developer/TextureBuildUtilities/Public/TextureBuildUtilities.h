// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ImageCore.h"
#include "Interfaces/ITextureFormat.h"
#include "Engine/TextureDefines.h"

#define UE_API TEXTUREBUILDUTILITIES_API

struct FTextureBuildSettings;

/***

TextureBuildUtilities is a utility module for shared code that Engine and TextureBuildWorker (no Engine) can both see

for Texture-related functions that don't need Texture.h/UTexture

TextureBuildUtilities is hard-linked to the Engine so no LoadModule/ModuleInterface is needed

***/

namespace UE
{
namespace TextureBuildUtilities
{
	enum
	{
		// The width and height of the placeholder gpu texture we create when the texture is cpu accessible.
		PLACEHOLDER_TEXTURE_SIZE = 4
	};


namespace EncodedTextureExtendedData
{
	TEXTUREBUILDUTILITIES_API FCbObject ToCompactBinary(const FEncodedTextureExtendedData& InExtendedData);
	TEXTUREBUILDUTILITIES_API bool FromCompactBinary(FEncodedTextureExtendedData& OutExtendedData, FCbObject InCbObject);
}

namespace EncodedTextureDescription
{
	TEXTUREBUILDUTILITIES_API FCbObject ToCompactBinary(const FEncodedTextureDescription& InDescription);
	TEXTUREBUILDUTILITIES_API bool FromCompactBinary(FEncodedTextureDescription& OutDescription, FCbObject InCbObject);
}

namespace TextureEngineParameters
{
	TEXTUREBUILDUTILITIES_API FCbObject ToCompactBinaryWithDefaults(const FTextureEngineParameters& InEngineParameters);
	TEXTUREBUILDUTILITIES_API bool FromCompactBinary(FTextureEngineParameters& OutEngineParameters, FCbObject InCbObject);
}

// Carries information out of the build that we don't want to cook or save off in the runtime
struct FTextureBuildMetadata
{
	// Digests of the data at various processing stages so we can track down determinism issues
	// that arise. Currently just the hash from before we pass to the encoders.
	uint64 PreEncodeMipsHash = 0;

	UE_API FCbObject ToCompactBinaryWithDefaults() const;
	UE_API FTextureBuildMetadata(FCbObject InCbObject);
	FTextureBuildMetadata() = default;
};

TEXTUREBUILDUTILITIES_API bool TextureFormatIsHdr(FName const& InName);

// Pass in the dimensions of the texture that will be created on the PC (i.e. take in to consideration whether
// LODBias mips will be stripped or not)
TEXTUREBUILDUTILITIES_API bool TextureNeedsDecodeForPC(EPixelFormat InPixelFormat, int32 InCreateMip0SizeX, int32 InCreateMip0SizeY);

// Removes platform and other custom prefixes from the name.
// Returns plain format name and the non-platform prefix (with trailing underscore).
// i.e. PLAT_BLAH_AutoDXT returns AutoDXT and writes BLAH_ to OutPrefix.
TEXTUREBUILDUTILITIES_API const FName TextureFormatRemovePrefixFromName(FName const& InName, FName& OutPrefix);

// removes platform prefix but leaves other custom prefixes :
TEXTUREBUILDUTILITIES_API const FName TextureFormatRemovePlatformPrefixFromName(FName const& InName);

FORCEINLINE const FName TextureFormatRemovePrefixFromName(FName const& InName)
{
	FName OutPrefix;
	return TextureFormatRemovePrefixFromName(InName,OutPrefix);
}

// Get the format to use for output of the VT Intermediate stage, cutting into tiles and processing
//	  the next step will then encode from this format to the desired output format
TEXTUREBUILDUTILITIES_API ERawImageFormat::Type GetVirtualTextureBuildIntermediateFormat(const FTextureBuildSettings& BuildSettings);

TEXTUREBUILDUTILITIES_API void GetPlaceholderTextureImageInfo(FImageInfo* OutImageInfo);
TEXTUREBUILDUTILITIES_API void GetPlaceholderTextureImage(FImage* OutImage);

// Returns true if the target texture size is different and padding/stretching is required.
//	if InPow2Setting == None, the Out sizes match the In sizes, and false is returned
TEXTUREBUILDUTILITIES_API bool GetPowerOfTwoTargetTextureSize(int32 InMip0SizeX, int32 InMip0SizeY, int32 InMip0NumSlices, bool bInIsVolume, ETexturePowerOfTwoSetting::Type InPow2Setting, int32 InResizeDuringBuildX, int32 InResizeDuringBuildY, int32& OutTargetSizeX, int32& OutTargetSizeY, int32& OutTargetSizeZ);

// Return the final output pixel format. When we don't know the source's alpha channel information this might not be knowable. In those cases
// treat such sources as having an alpha channel if bKnownAlphaFallback is true.
TEXTUREBUILDUTILITIES_API EPixelFormat GetOutputPixelFormatWithFallback(const FTextureBuildSettings& InBuildSettings, bool bInKnownAlphaFallback);

// Return the estimate for how much memory a physical texture will take to build, for managing memory resources during build dispatch.
TEXTUREBUILDUTILITIES_API int64 GetPhysicalTextureBuildMemoryEstimate(const FTextureBuildSettings* InBuildSettings, const FImageInfo& InSourceImageInfo, int32 InMipCount);

// Mirrors Texture.h FTextureSourceBlock for use without Texture.h
//
// This provides the layout representation of a block of source pixels for a virtual texture. The format is separate
// as VTs can have multiple layers, each with the same source pixel layout but with a different format (and thus byte size).
struct FVirtualTextureSourceBlockInfo
{
	// @@ I don't actually know what coordinate space blocks are in and none of the code I'm looking at seems very clear on it.
	// afaict it's just the index of the source image and where it exists. I don't know what happens if you have a hole in your source blocks.
	int32 BlockX=0;
	int32 BlockY=0;

	// Pixel dims.
	int32 SizeX=0;
	int32 SizeY=0;

	// afaict this is ignored: see "BlockData.NumSlices = 1; // TODO?" in VirtualTextureDataBuilder
	int32 NumSlices=0;

	int32 NumMips=0;
};

// Return the estimate for how much memory a virtual texture will take to build, for managing memory resources during build dispatch.
TEXTUREBUILDUTILITIES_API int64 GetVirtualTextureRequiredMemoryEstimate(const FTextureBuildSettings* InBuildSettingsPerLayer,
	TConstArrayView<ERawImageFormat::Type> InLayerFormats,
	TConstArrayView<UE::TextureBuildUtilities::FVirtualTextureSourceBlockInfo> InSourceBlocks);
	
// ComputeLongLatCubemapExtents is done after pad-to-pow2
TEXTUREBUILDUTILITIES_API uint32 ComputeLongLatCubemapExtents(int32 SrcImageSizeX, uint32 MaxCubemapTextureResolution);

} // namespace TextureBuildUtilities
} // namespace UE

#undef UE_API
