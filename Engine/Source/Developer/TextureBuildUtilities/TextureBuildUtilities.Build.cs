// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TextureBuildUtilities : ModuleRules
{
	public TextureBuildUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
		// TextureBuildUtilities can be used in TBW so cannot depend on Engine

		// Include only , no link :
		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"TextureCompressor",
			"Engine"
		});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",			
				"ImageCore",
				"TextureFormat",
			}
			);

	}
}
